{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Mutable Dataflow Extensions",
	"Description": "Adds Mutable functionality to work with Dataflow objects from the Dataflow plugin. WARNING: All nodes in this plugin are experimental and will be changed/deprecated in the future. The Dataflow system does not yet fully support mutable so the functionality of these nodes is currently quite limited and manual.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": 
	[
		{
			"Name": "MutableDataflowEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": 
	[
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "Mutable",
			"Enabled": true
		}
	]

}