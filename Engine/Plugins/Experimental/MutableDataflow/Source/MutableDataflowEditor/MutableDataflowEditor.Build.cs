// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MutableDataflowEditor : ModuleRules
	{
        public MutableDataflowEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.Add(ModuleDirectory + "/Private");
			PublicIncludePaths.Add(ModuleDirectory + "/Public");
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"CustomizableObject",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DataflowCore",
					"DataflowEditor",
					"DataflowEngine",
				});
		}
	}
}
