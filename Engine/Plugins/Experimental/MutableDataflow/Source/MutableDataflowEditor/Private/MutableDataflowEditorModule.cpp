// Copyright Epic Games, Inc. All Rights Reserved.

#include "MutableDataflowEditorModule.h"

#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowCore.h"
#include "Nodes/COInstanceGeneratorNode.h"
#include "Nodes/MakeMutableBoolParametersArrayNode.h"
#include "Nodes/MakeMutableEnumParametersArrayNode.h"
#include "Nodes/MakeMutableFloatParametersArrayNode.h"
#include "Nodes/MakeMutableInstancedStructParametersArrayNode.h"
#include "Nodes/MutableMaterialParameterNode.h"
#include "Nodes/MutableSkeletalMeshParameterNode.h"
#include "Nodes/MutableTextureParameterNode.h"
#include "Nodes/MakeMutableMaterialParametersArrayNode.h"
#include "Nodes/MakeMutableProjectorParametersArrayNode.h"
#include "Nodes/MakeMutableSkeletalMeshParametersArrayNode.h"
#include "Nodes/MakeMutableTextureParametersArrayNode.h"
#include "Nodes/MakeMutableTransformParametersArrayNode.h"
#include "Nodes/MakeMutableVectorParametersArrayNode.h"
#include "Nodes/MutableBoolParameterNode.h"
#include "Nodes/MutableEnumParameterNode.h"
#include "Nodes/MutableFloatParameterNode.h"
#include "Nodes/MutableProjectorParameterNode.h"
#include "Nodes/MutableTransformParameterNode.h"
#include "Nodes/MutableVectorParameterNode.h"
#include "Nodes/MutableInstancedStructParameterNode.h"

IMPLEMENT_MODULE(FMutableDataflowEditorModule, MutableDataflowEditor);

void FMutableDataflowEditorModule::StartupModule()
{
	const FLinearColor MutableNodeTitleColor = FColor(234, 255, 0);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Mutable", MutableNodeTitleColor, UE::Dataflow::FColorScheme::NodeBody);
	
	// Main nodes
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCOInstanceGeneratorNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCOInstanceGetComponentMesh);

	// Auto connection between an array of FMutableGeneratedResource to a single TObjectPtr<USkeletalMesh>
	UE_DATAFLOW_REGISTER_AUTOCONVERT(TArray<FMutableGeneratedResource>, TObjectPtr<USkeletalMesh>, FCOInstanceGetComponentMesh);
	
	// Parameter nodes
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableSkeletalMeshParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableTextureParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableMaterialParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableBoolParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableFloatParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableProjectorParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableTransformParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableVectorParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableEnumParameterNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMutableInstancedStructParameterNode);

	// Parameter array generation nodes
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableSkeletalMeshParametersArrayNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableMaterialParametersArrayNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableTextureParametersArrayNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableBoolParametersArrayNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableFloatParametersArrayNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableProjectorParametersArrayNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableTransformParametersArrayNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableVectorParametersArrayNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableEnumParametersArrayNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMutableInstancedStructParametersArrayNode);

	// Parameter node to array automatic conversion nodes
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableSkeletalMeshParameter, TArray<FMutableSkeletalMeshParameter>, FMakeMutableSkeletalMeshParametersArrayNode);
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableMaterialParameter, TArray<FMutableMaterialParameter>, FMakeMutableMaterialParametersArrayNode);
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableTextureParameter, TArray<FMutableTextureParameter>, FMakeMutableTextureParametersArrayNode);
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableBoolParameter, TArray<FMutableBoolParameter>, FMakeMutableBoolParametersArrayNode);
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableFloatParameter, TArray<FMutableFloatParameter>, FMakeMutableFloatParametersArrayNode);
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableProjectorParameter, TArray<FMutableProjectorParameter>, FMakeMutableProjectorParametersArrayNode);
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableTransformParameter, TArray<FMutableTransformParameter>, FMakeMutableTransformParametersArrayNode);
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableVectorParameter, TArray<FMutableVectorParameter>, FMakeMutableVectorParametersArrayNode);
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableEnumParameter, TArray<FMutableEnumParameter>, FMakeMutableEnumParametersArrayNode);
	UE_DATAFLOW_REGISTER_AUTOCONVERT(FMutableInstancedStructParameter, TArray<FMutableInstancedStructParameter>, FMakeMutableInstancedStructParametersArrayNode);

}
