// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Public/Nodes/COInstanceGeneratorNode.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "Engine/SkeletalMesh.h"
#include "MutableDataflowParameters.h"


FCOInstanceGetComponentMesh::FCOInstanceGetComponentMesh(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&GeneratedResources);
	RegisterInputConnection(&ComponentName);

	RegisterOutputConnection(&ComponentSkeletalMesh);
}


void FCOInstanceGetComponentMesh::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	const TArray<FMutableGeneratedResource> InputGeneratedResources = GetValue(Context, &GeneratedResources);
	const FString TargetComponentName = GetValue(Context, &ComponentName);

	TObjectPtr<USkeletalMesh> OutputSkeletalMesh = nullptr;
	if (!TargetComponentName.IsEmpty())
	{
		for (const FMutableGeneratedResource& Resource : InputGeneratedResources)
		{
			if (Resource.ComponentName.Equals(TargetComponentName))
			{
				OutputSkeletalMesh = Resource.SkeletalMesh;
				break;
			}
		}
	}
	
	SetValue(Context, OutputSkeletalMesh, &ComponentSkeletalMesh);
}



FCOInstanceGeneratorNode::FCOInstanceGeneratorNode(const UE::Dataflow::FNodeParameters& InParam,FGuid InGuid)
: FDataflowNode(InParam, InGuid),
	GenerateInstanceResources( FDataflowFunctionProperty::FDelegate::CreateRaw(this, &FCOInstanceGeneratorNode::OnGenerateInstanceResourcesRequested))
{
	// Inputs
	RegisterInputConnection(&CustomizableObject);
	
	RegisterInputConnection(&SkeletalMeshParameters);
	RegisterInputConnection(&TextureParameters);
	RegisterInputConnection(&MaterialParameters);
	RegisterInputConnection(&BoolParameters);
	RegisterInputConnection(&EnumParameters);
	RegisterInputConnection(&FloatParameters);
	RegisterInputConnection(&VectorParameters);
	RegisterInputConnection(&ProjectorParameters);
	RegisterInputConnection(&TransformParameters);
	RegisterInputConnection(&InstancedStructParameters);
	
	// Outputs
	RegisterOutputConnection(&GeneratedResources);
	RegisterOutputConnection(&GeneratedSkeletalMeshes);
}


void FCOInstanceGeneratorNode::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent)
{
	FDataflowNode::OnPropertyChanged(Context, InPropertyChangedEvent);

	if (const FProperty* ModifiedProperty = InPropertyChangedEvent.Property)
	{
		if (ModifiedProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FCOInstanceGeneratorNode, CustomizableObject))
		{
			ClearCachedParameters();
			
			CustomizableObjectInstance = nullptr;
			
			GeneratedResources.Reset();
			GeneratedSkeletalMeshes.Reset();
			Invalidate();
		}
	}
}


void FCOInstanceGeneratorNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (!GetValue(Context, &CustomizableObject))
	{
		Context.Error(TEXT("Unable to evaluate : No Customizable Object has been set"), this);
		return;
	}
	
	if (!CustomizableObjectInstance)
	{
		Context.Error(TEXT("Unable to evaluate : Press \"Generate Resources\" before evaluating the node"), this);
		return;
	}

	if (bIsGeneratingResources)
	{
		Context.Warning(TEXT("Unable to evaluate : Resource generation is in process. Please hold"), this);
		return;
	}
	
	if (Out->IsA<TArray<FMutableGeneratedResource>>(&GeneratedResources))
	{
		SetValue(Context, GeneratedResources, &GeneratedResources);
	}
	else if (Out->IsA<TArray<TObjectPtr<USkeletalMesh>>>(&GeneratedSkeletalMeshes))
	{
		// Extract the generated meshes from the generated resources collection as there we have them already set
		TArray<TObjectPtr<USkeletalMesh>> SkeletalMeshes;
		SkeletalMeshes.Reserve(GeneratedResources.Num());
		for (const FMutableGeneratedResource& Resource : GeneratedResources)
		{
			SkeletalMeshes.Add(Resource.SkeletalMesh);
		}
		SetValue(Context, SkeletalMeshes, &GeneratedSkeletalMeshes);
	}
}


void FCOInstanceGeneratorNode::ClearCachedParameters()
{
	CachedSkeletalMeshParameters.Reset();
	CachedMaterialParameters.Reset();
	CachedTextureParameters.Reset();
	CachedBoolParameters.Reset();
	CachedEnumParameters.Reset();
	CachedFloatParameters.Reset();
	CachedVectorParameters.Reset();
	CachedProjectorParameters.Reset();
	CachedTransformParameters.Reset();
	CachedInstancedStructParameters.Reset();
}


void FCOInstanceGeneratorNode::OnGenerateInstanceResourcesRequested(UE::Dataflow::FContext& Context)
{
	if (bIsGeneratingResources)
	{
		Context.Warning(TEXT("Unable to generate resources. Resource generation is already in process"), this);
		return;
	}
	
	if (GetValue(Context, &CustomizableObject))
	{
		// Prevent the scheduling or more compilations while the generation of resources is in process
		bIsGeneratingResources = true;
		RequestCompilation(Context);
	}
	else
	{
		Context.Error(TEXT("Unable to generate resources. The Customizable object has not been set"), this);
	}
}


void FCOInstanceGeneratorNode::RequestCompilation(UE::Dataflow::FContext& Context)
{
	// Capture the inputs of the node (target parameters) so we can apply them after the compilation without us having to rely on the context (as it may be gone)
	CacheNodeInputs(Context);
	check(CustomizableObject);
	
	FCompileNativeDelegate CompilationDelegate;
	TSharedRef<FDataflowNode> This = AsShared();
	CompilationDelegate.BindLambda([This](const FCompileCallbackParams& CallbackParams)->void
	{
		FCOInstanceGeneratorNode* HostNode = static_cast<FCOInstanceGeneratorNode*>(&This.Get());
		check(HostNode);

		// Early out if the request failed
		if (CallbackParams.bRequestFailed || CallbackParams.bErrors)
		{
			// Tell the system it is ok to generate resources again
			HostNode->bIsGeneratingResources = false;
			return;
		}
			
		// First generate the instance
		HostNode->CustomizableObjectInstance = HostNode->CustomizableObject->CreateInstance();
		check(HostNode->CustomizableObjectInstance)
			
		// Apply the parameters set in the node into the instance
		HostNode->ApplyInstanceParameters(*HostNode->CustomizableObjectInstance);

		// once the compilation has been performed (if required) ask for the update of the COI
		HostNode->RequestUpdate(*HostNode->CustomizableObjectInstance);
	});

	// Request the compilation of the CO
	FCompileParams CompilationParams;
	CompilationParams.bAsync = true;
	CompilationParams.CallbackNative = CompilationDelegate;
	
	// todo: UE-313428 Expose compilation errors and warnings to the Dataflow context
	CustomizableObject->Compile(CompilationParams);

	// Prevent the scheduling or more compilations while the generation of resources is in process
	bIsGeneratingResources = true;
}


void FCOInstanceGeneratorNode::CacheNodeInputs(UE::Dataflow::FContext& Context)
{
	ClearCachedParameters();

	// Customizable Object
	CustomizableObject = GetValue(Context, &CustomizableObject);

	// Cache the parameters now that we have the context. Do not cache duplicate entries (based on the param name
	CachedTextureParameters = GetPropertyParameters(&TextureParameters, TEXT("Texture"), Context);
	CachedSkeletalMeshParameters = GetPropertyParameters(&SkeletalMeshParameters,TEXT("Skeletal Mesh"), Context);
	CachedMaterialParameters = GetPropertyParameters(&MaterialParameters,TEXT("Material"), Context);
	CachedBoolParameters = GetPropertyParameters(&BoolParameters,TEXT("Bool"), Context);
	CachedEnumParameters = GetPropertyParameters(&EnumParameters,TEXT("Enum"), Context);
	CachedFloatParameters = GetPropertyParameters(&FloatParameters,TEXT("Float"), Context);
	CachedVectorParameters = GetPropertyParameters(&VectorParameters,TEXT("Vector"), Context);
	CachedProjectorParameters = GetPropertyParameters(&ProjectorParameters,TEXT("Projector"), Context);
	CachedTransformParameters = GetPropertyParameters(&TransformParameters,TEXT("Transform"),Context);
	CachedInstancedStructParameters = GetPropertyParameters(&InstancedStructParameters,TEXT("Instanced Struct"), Context);
}


void FCOInstanceGeneratorNode::ApplyInstanceParameters(UCustomizableObjectInstance& Instance)
{
	// todo. notify when a parameter does not exist :
	// In order to do that we will require to have async node evaluations and then access to the
	// context to log the warning
	
	for (const FMutableTextureParameter& TextureParameter : CachedTextureParameters)
	{
		const FString ParameterName = TextureParameter.Name;
		if (Instance.ContainsTextureParameter(ParameterName))
		{
			Instance.SetTextureParameterSelectedOption(ParameterName, TextureParameter.Texture);
		}
	}
	
	for (const FMutableSkeletalMeshParameter& MeshParameter : CachedSkeletalMeshParameters)
	{
		const FString ParameterName = MeshParameter.Name;
		if (Instance.ContainsSkeletalMeshParameter(ParameterName))
		{
			Instance.SetSkeletalMeshParameterSelectedOption(ParameterName, MeshParameter.Mesh);
		}
	}
	
	for (const FMutableMaterialParameter& MaterialParameter : CachedMaterialParameters)
	{
		const FString ParameterName = MaterialParameter.Name;
		if (Instance.ContainsMaterialParameter(ParameterName))
		{
			Instance.SetMaterialParameterSelectedOption(ParameterName, MaterialParameter.Material);				
		}
	}
	
	for (const FMutableBoolParameter& BoolParameter : CachedBoolParameters)
	{
		const FString ParameterName = BoolParameter.Name;
		if (Instance.ContainsBoolParameter(ParameterName))
		{
			Instance.SetBoolParameterSelectedOption(ParameterName, BoolParameter.Bool);				
		}
	}

	for (const FMutableEnumParameter& EnumParameter : CachedEnumParameters)
	{
		const FString ParameterName = EnumParameter.Name;
		if (Instance.ContainsEnumParameter(ParameterName))
		{
			Instance.SetEnumParameterSelectedOption(ParameterName, EnumParameter.OptionName);				
		}
	}

	for (const FMutableFloatParameter& FloatParameter : CachedFloatParameters)
	{
		const FString ParameterName = FloatParameter.Name;
		if (Instance.ContainsFloatParameter(ParameterName))
		{
			Instance.SetFloatParameterSelectedOption(ParameterName, FloatParameter.Float);				
		}
	}

	for (const FMutableVectorParameter& VectorParameter : CachedVectorParameters)
	{
		const FString ParameterName = VectorParameter.Name;
		if (Instance.ContainsVectorParameter(ParameterName))
		{
			Instance.SetVectorParameterSelectedOption(ParameterName, VectorParameter.Color);				
		}
	}
	
	for (const FMutableProjectorParameter& ProjectorParameter : CachedProjectorParameters)
	{
		const FString ParameterName = ProjectorParameter.Name;
		if (Instance.ContainsProjectorParameter(ParameterName))
		{
			Instance.SetProjectorParameterSelectedOption(ParameterName, ProjectorParameter.Projector);
		}
	}
	
	for (const FMutableTransformParameter& TransformParameter : CachedTransformParameters)
	{
		const FString ParameterName = TransformParameter.Name;
		if (Instance.ContainsTransformParameter(ParameterName))
		{
			Instance.SetTransformParameterSelectedOption(ParameterName, TransformParameter.Transform);				
		}
	}
	
	for (const FMutableInstancedStructParameter& InstancedStructParameter : CachedInstancedStructParameters)
	{
		const FString ParameterName = InstancedStructParameter.Name;
		if (Instance.ContainsExternalTypeParameter(ParameterName))
		{
			Instance.SetExternalTypeParameterSelectedOption(ParameterName, InstancedStructParameter.InstancedStruct);				
		}
	}
}


void FCOInstanceGeneratorNode::RequestUpdate(UCustomizableObjectInstance& Instance)
{
	TSharedRef<FDataflowNode> This = AsShared();
	
	// Instance update delegate	
	FInstanceUpdateNativeDelegate InstanceUpdateNativeDelegate;
	InstanceUpdateNativeDelegate.AddLambda([This](FUpdateContext OutUpdateContext) -> void
	{
		FCOInstanceGeneratorNode* HostNode = static_cast<FCOInstanceGeneratorNode*>(&This.Get());
		check(HostNode);
			
		HostNode->GeneratedResources = HostNode->GetInstanceGeneratedMeshes(*OutUpdateContext.Instance);
			
		// Tell the system it is ok to generate resources again
		HostNode->bIsGeneratingResources = false;
			
		This->Invalidate();
	});
	 		
	Instance.UpdateSkeletalMeshAsyncResult(InstanceUpdateNativeDelegate,true, true);
}


TArray<FMutableGeneratedResource> FCOInstanceGeneratorNode::GetInstanceGeneratedMeshes(const UCustomizableObjectInstance& Instance) const
{
	TArray<FMutableGeneratedResource> GeneratedMeshesData;
			
	const TArray<FName> InstanceComponentNames = Instance.GetComponentNames();
	for (const FName& ComponentName : InstanceComponentNames)
	{
		if (const TObjectPtr<USkeletalMesh> SkeletalMesh = Instance.GetSkeletalMeshComponentSkeletalMesh(ComponentName))
		{
			FMutableGeneratedResource ComponentData;
			ComponentData.ComponentName = ComponentName.ToString();
			ComponentData.SkeletalMesh = SkeletalMesh;
			
			GeneratedMeshesData.Add(ComponentData);
		}
	}

	return GeneratedMeshesData;
}

