// Copyright Epic Games, Inc. All Rights Reserved.


#include "Nodes/MakeMutableBoolParametersArrayNode.h"


FMakeMutableBoolParametersArrayNode::FMakeMutableBoolParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputBoolParameters);
}


void FMakeMutableBoolParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputBoolParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableBoolParametersArrayNode::AddPins()
{
	return AddParameterPin(InputBoolParameters);
}


bool FMakeMutableBoolParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputBoolParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableBoolParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputBoolParameters);
}


void FMakeMutableBoolParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputBoolParameters, Pin);
}


void FMakeMutableBoolParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputBoolParameters, Ar);
}
