// Copyright Epic Games, Inc. All Rights Reserved.


#include "Nodes/MakeMutableEnumParametersArrayNode.h"


FMakeMutableEnumParametersArrayNode::FMakeMutableEnumParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputEnumParameters);
}


void FMakeMutableEnumParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputEnumParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableEnumParametersArrayNode::AddPins()
{
	return AddParameterPin(InputEnumParameters);
}


bool FMakeMutableEnumParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputEnumParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableEnumParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputEnumParameters);
}


void FMakeMutableEnumParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputEnumParameters, Pin);
}


void FMakeMutableEnumParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputEnumParameters, Ar);
}
