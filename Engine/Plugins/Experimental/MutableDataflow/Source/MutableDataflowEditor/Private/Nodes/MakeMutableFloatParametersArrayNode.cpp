// Copyright Epic Games, Inc. All Rights Reserved.


#include "Nodes/MakeMutableFloatParametersArrayNode.h"


FMakeMutableFloatParametersArrayNode::FMakeMutableFloatParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputFloatParameters);
}


void FMakeMutableFloatParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputFloatParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableFloatParametersArrayNode::AddPins()
{
	return AddParameterPin(InputFloatParameters);
}


bool FMakeMutableFloatParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputFloatParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableFloatParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputFloatParameters);
}


void FMakeMutableFloatParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputFloatParameters, Pin);
}


void FMakeMutableFloatParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputFloatParameters, Ar);
}
