// Copyright Epic Games, Inc. All Rights Reserved.


#include "../../Public/Nodes/MakeMutableInstancedStructParametersArrayNode.h"
#include "Nodes/MakeMutableInstancedStructParametersArrayNode.h"
#include "Dataflow/DataflowNode.h"


FMakeMutableInstancedStructParametersArrayNode::FMakeMutableInstancedStructParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputStructParameters);
}


void FMakeMutableInstancedStructParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputStructParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableInstancedStructParametersArrayNode::AddPins()
{
	return AddParameterPin(InputStructParameters);
}


bool FMakeMutableInstancedStructParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputStructParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableInstancedStructParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputStructParameters);
}


void FMakeMutableInstancedStructParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputStructParameters, Pin);
}


void FMakeMutableInstancedStructParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputStructParameters, Ar);
}
