// Copyright Epic Games, Inc. All Rights Reserved.


#include "Nodes/MakeMutableMaterialParametersArrayNode.h"


FMakeMutableMaterialParametersArrayNode::FMakeMutableMaterialParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputMaterialParameters);
}


void FMakeMutableMaterialParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputMaterialParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableMaterialParametersArrayNode::AddPins()
{
	return AddParameterPin(InputMaterialParameters);
}


bool FMakeMutableMaterialParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputMaterialParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableMaterialParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputMaterialParameters);
}


void FMakeMutableMaterialParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputMaterialParameters, Pin);
}


void FMakeMutableMaterialParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputMaterialParameters, Ar);
}
