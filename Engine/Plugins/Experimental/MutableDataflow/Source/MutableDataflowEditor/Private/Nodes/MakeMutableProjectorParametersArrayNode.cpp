// Copyright Epic Games, Inc. All Rights Reserved.


#include "Nodes/MakeMutableProjectorParametersArrayNode.h"


FMakeMutableProjectorParametersArrayNode::FMakeMutableProjectorParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputProjectorParameters);
}


void FMakeMutableProjectorParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputProjectorParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableProjectorParametersArrayNode::AddPins()
{
	return AddParameterPin(InputProjectorParameters);
}


bool FMakeMutableProjectorParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputProjectorParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableProjectorParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputProjectorParameters);
}


void FMakeMutableProjectorParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputProjectorParameters, Pin);
}


void FMakeMutableProjectorParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputProjectorParameters, Ar);
}
