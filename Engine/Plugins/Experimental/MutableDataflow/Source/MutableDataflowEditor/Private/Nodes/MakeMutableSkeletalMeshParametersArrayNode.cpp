// Copyright Epic Games, Inc. All Rights Reserved.


#include "Nodes/MakeMutableSkeletalMeshParametersArrayNode.h"


FMakeMutableSkeletalMeshParametersArrayNode::FMakeMutableSkeletalMeshParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputSkeletalMeshParameters);
}


void FMakeMutableSkeletalMeshParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputSkeletalMeshParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableSkeletalMeshParametersArrayNode::AddPins()
{
	return AddParameterPin(InputSkeletalMeshParameters);
}


bool FMakeMutableSkeletalMeshParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputSkeletalMeshParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableSkeletalMeshParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputSkeletalMeshParameters);
}


void FMakeMutableSkeletalMeshParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputSkeletalMeshParameters, Pin);
}


void FMakeMutableSkeletalMeshParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputSkeletalMeshParameters, Ar);
}
