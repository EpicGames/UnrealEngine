// Copyright Epic Games, Inc. All Rights Reserved.


#include "Nodes/MakeMutableTextureParametersArrayNode.h"


FMakeMutableTextureParametersArrayNode::FMakeMutableTextureParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputTextureParameters);
}


void FMakeMutableTextureParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputTextureParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableTextureParametersArrayNode::AddPins()
{
	return AddParameterPin(InputTextureParameters);
}


bool FMakeMutableTextureParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputTextureParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableTextureParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputTextureParameters);
}


void FMakeMutableTextureParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputTextureParameters, Pin);
}


void FMakeMutableTextureParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputTextureParameters, Ar);
}
