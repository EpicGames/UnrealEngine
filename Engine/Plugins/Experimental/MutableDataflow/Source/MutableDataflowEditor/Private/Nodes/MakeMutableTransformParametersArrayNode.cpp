// Copyright Epic Games, Inc. All Rights Reserved.


#include "Nodes/MakeMutableTransformParametersArrayNode.h"


FMakeMutableTransformParametersArrayNode::FMakeMutableTransformParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputTransformParameters);
}


void FMakeMutableTransformParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputTransformParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableTransformParametersArrayNode::AddPins()
{
	return AddParameterPin(InputTransformParameters);
}


bool FMakeMutableTransformParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputTransformParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableTransformParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputTransformParameters);
}


void FMakeMutableTransformParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputTransformParameters, Pin);
}


void FMakeMutableTransformParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputTransformParameters, Ar);
}
