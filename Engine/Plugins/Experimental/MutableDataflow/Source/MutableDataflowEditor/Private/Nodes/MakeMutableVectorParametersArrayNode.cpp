// Copyright Epic Games, Inc. All Rights Reserved.


#include "Nodes/MakeMutableVectorParametersArrayNode.h"


FMakeMutableVectorParametersArrayNode::FMakeMutableVectorParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterOutputConnection(&GroupedParameters);

	// Create as many inputs as the node requires to have by default
	AddDefaultInputs(InputVectorParameters);
}


void FMakeMutableVectorParametersArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	EvaluateParameterNode(InputVectorParameters, GroupedParameters, Context, Out);
}


TArray<UE::Dataflow::FPin> FMakeMutableVectorParametersArrayNode::AddPins()
{
	return AddParameterPin(InputVectorParameters);
}


bool FMakeMutableVectorParametersArrayNode::CanRemovePin() const
{
	return CanRemoveParameterPin(InputVectorParameters);
}


TArray<UE::Dataflow::FPin> FMakeMutableVectorParametersArrayNode::GetPinsToRemove() const
{
	return GetParameterPinsToRemove(InputVectorParameters);
}


void FMakeMutableVectorParametersArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	OnParameterPinRemoved(InputVectorParameters, Pin);
}


void FMakeMutableVectorParametersArrayNode::PostSerialize(const FArchive& Ar)
{
	PostNodeSerialize(InputVectorParameters, Ar);
}
