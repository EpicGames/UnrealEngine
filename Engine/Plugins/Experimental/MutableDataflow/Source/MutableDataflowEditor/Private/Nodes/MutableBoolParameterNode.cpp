// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableBoolParameterNode.h"

#include "Dataflow/DataflowObject.h"


FMutableBoolParameterNode::FMutableBoolParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&Bool);

	RegisterOutputConnection(&BoolParameter);
}


void FMutableBoolParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoolParameter))
	{
		FMutableBoolParameter Output;
		Output.Name = GetParameterName(Context);
		Output.Bool = GetValue(Context, &Bool);
	
		SetValue(Context, Output, &BoolParameter);
	}
}