// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableEnumParameterNode.h"

#include "Dataflow/DataflowObject.h"

FMutableEnumParameterNode::FMutableEnumParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&EnumOption);

	RegisterOutputConnection(&EnumParameter);
}


void FMutableEnumParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&EnumParameter))
	{
		FMutableEnumParameter Output;
		Output.Name = GetParameterName(Context);
		Output.OptionName = GetValue(Context, &EnumOption);
	
		SetValue(Context, Output, &EnumParameter);
	}
}