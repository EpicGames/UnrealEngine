// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableFloatParameterNode.h"

#include "Dataflow/DataflowObject.h"

FMutableFloatParameterNode::FMutableFloatParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&Float);

	RegisterOutputConnection(&FloatParameter);
}


void FMutableFloatParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatParameter))
	{
		FMutableFloatParameter Output;
		Output.Name = GetParameterName(Context);
		Output.Float = GetValue(Context, &Float);
	
		SetValue(Context, Output, &FloatParameter);
	}
}