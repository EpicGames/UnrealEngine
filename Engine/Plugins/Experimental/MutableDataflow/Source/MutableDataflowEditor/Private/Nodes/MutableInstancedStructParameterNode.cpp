// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableInstancedStructParameterNode.h"

#include "Dataflow/DataflowEngineUtil.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowAnyType.h"


FMutableInstancedStructParameterNode::FMutableInstancedStructParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&InputStruct);
	
	RegisterOutputConnection(&InstancedStructParameter);
}


void FMutableInstancedStructParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&InstancedStructParameter))
	{
		FMutableInstancedStructParameter Output;
		Output.Name = GetParameterName(Context);
		Output.InstancedStruct = GetValue(Context, &InputStruct).GetConstStructView();
	
		SetValue(Context, Output, &InstancedStructParameter);
	}
}
