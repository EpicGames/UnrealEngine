// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableMaterialParameterNode.h"
#include "Materials/MaterialInterface.h"

FMutableMaterialParameterNode::FMutableMaterialParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
: Super(InParam, InGuid)
{
	RegisterInputConnection(&Material);

	RegisterOutputConnection(&MaterialParameter);
}


void FMutableMaterialParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MaterialParameter))
	{
		FMutableMaterialParameter Output;
		Output.Name = GetParameterName(Context);
		Output.Material = GetValue(Context, &Material);
	
		SetValue(Context, Output, &MaterialParameter);
	}
}
