// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableParameterNode.h"
#include "Dataflow/DataflowObject.h"

FMutableParameterNode::FMutableParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&ParameterName);
}

FString FMutableParameterNode::GetParameterName(UE::Dataflow::FContext& Context) const
{
	const FString OutParameterName = GetValue(Context, &ParameterName);
	
	if (OutParameterName.IsEmpty())
	{
		Context.Warning(TEXT("The generated Mutable Parameter name is empty."), this);
	}
	
	return OutParameterName;
}