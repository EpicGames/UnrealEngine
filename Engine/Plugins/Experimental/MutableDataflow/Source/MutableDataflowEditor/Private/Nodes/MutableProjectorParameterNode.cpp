// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableProjectorParameterNode.h"

#include "Dataflow/DataflowObject.h"


FMutableProjectorParameterNode::FMutableProjectorParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&Projector);

	RegisterOutputConnection(&ProjectorParameter);
}


void FMutableProjectorParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&ProjectorParameter))
	{
		FMutableProjectorParameter Output;
		Output.Name = GetParameterName(Context);
		Output.Projector = GetValue(Context, &Projector);
	
		SetValue(Context, Output, &ProjectorParameter);
	}
}