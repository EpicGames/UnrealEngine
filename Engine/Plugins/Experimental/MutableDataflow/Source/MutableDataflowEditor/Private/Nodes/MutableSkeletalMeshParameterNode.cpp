// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableSkeletalMeshParameterNode.h"

#include "Dataflow/DataflowObject.h"
#include "Engine/SkeletalMesh.h"

FMutableSkeletalMeshParameterNode::FMutableSkeletalMeshParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&SkeletalMesh);

	RegisterOutputConnection(&SkeletalMeshParameter);
}


void FMutableSkeletalMeshParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SkeletalMeshParameter))
	{
		FMutableSkeletalMeshParameter Output;
		Output.Name = GetParameterName(Context);
		Output.Mesh = GetValue(Context, &SkeletalMesh);
	
		SetValue(Context, Output, &SkeletalMeshParameter);
	}
}

