// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableTextureParameterNode.h"
#include "Engine/Texture2D.h"

FMutableTextureParameterNode::FMutableTextureParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&Texture);

	RegisterOutputConnection(&TextureParameter);
}


void FMutableTextureParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TextureParameter))
	{
		FMutableTextureParameter Output;
		Output.Name =  GetParameterName(Context);
		Output.Texture = GetValue(Context, &Texture);

		SetValue(Context, Output, &TextureParameter);
	}
}