// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableTransformParameterNode.h"

#include "Dataflow/DataflowObject.h"


FMutableTransformParameterNode::FMutableTransformParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&Transform);

	RegisterOutputConnection(&TransformParameter);
}


void FMutableTransformParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TransformParameter))
	{
		FMutableTransformParameter Output;
		Output.Name = GetParameterName(Context);
		Output.Transform = GetValue(Context, &Transform);
	
		SetValue(Context, Output, &TransformParameter);
	}
}