// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/MutableVectorParameterNode.h"

#include "Dataflow/DataflowObject.h"


FMutableVectorParameterNode::FMutableVectorParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
: Super(InParam, InGuid)
{
	RegisterInputConnection(&Color);

	RegisterOutputConnection(&VectorParameter);
}


void FMutableVectorParameterNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&VectorParameter))
	{
		FMutableVectorParameter Output;
		Output.Name = GetParameterName(Context);
		Output.Color = GetValue(Context, &Color);
	
		SetValue(Context, Output, &VectorParameter);
	}
}