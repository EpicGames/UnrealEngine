// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "MuCO/CustomizableObjectParameterTypeDefinitions.h"

#include "MutableDataflowParameters.generated.h"

class USkeletalMesh;
class UTexture2D;
class UMaterialInterface;

USTRUCT()
struct FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()

	virtual ~FMutableParameterBase() = default;
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	FString Name;
	
	bool operator== (const FMutableParameterBase& Other) const
	{
		return this->Name.ToLower() == Other.Name.ToLower();
	}
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableSkeletalMeshParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	TObjectPtr<USkeletalMesh> Mesh;
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableTextureParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	TObjectPtr<UTexture2D> Texture;
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableMaterialParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	TObjectPtr<UMaterialInterface> Material;
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableBoolParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	bool Bool;
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableEnumParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	FString OptionName;
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableFloatParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	float Float;
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableVectorParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	FLinearColor Color;
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableProjectorParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Mutable")
	FCustomizableObjectProjector Projector;
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableTransformParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	FTransform Transform;
};


USTRUCT(BlueprintType, meta = (Experimental))
struct FMutableInstancedStructParameter : public FMutableParameterBase
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Mutable")
	FInstancedStruct InstancedStruct; 
};
