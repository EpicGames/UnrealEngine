// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowNode.h"
#include "Misc/Guid.h"
#include "CoreMinimal.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "MuCO/CustomizableObject.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowInputOutput.h"
#include "MutableDataflowParameters.h"

#include "COInstanceGeneratorNode.generated.h"

USTRUCT(meta = (Experimental))
struct FMutableGeneratedResource
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Mutable")
	FString ComponentName;

	UPROPERTY(EditAnywhere, Category = "Mutable")
	TObjectPtr<USkeletalMesh> SkeletalMesh;
};


USTRUCT(meta = (Experimental))
struct FCOInstanceGetComponentMesh : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCOInstanceGetComponentMesh, "GetComponentMesh", "Mutable", "")

	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DataflowInput))
	FString ComponentName;

	UPROPERTY(meta = (DataflowInput))
	TArray<FMutableGeneratedResource> GeneratedResources;

	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<USkeletalMesh> ComponentSkeletalMesh;

public:
	
	FCOInstanceGetComponentMesh(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/**
 * Node designed to work as an entry point for mutable in the DataFlow graph.
 * It allows the setup of a Customizable Object Instance and the extraction of the skeletal meshes generated by its update.
 */
USTRUCT(meta = (Experimental))
struct FCOInstanceGeneratorNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCOInstanceGeneratorNode, "GenerateCustomizableObjectInstance", "Mutable", "")

public:
	
	FCOInstanceGeneratorNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& InPropertyChangedEvent) override;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:
	
	// Inputs

	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DataflowInput))
	TObjectPtr<UCustomizableObject> CustomizableObject;

	
	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableSkeletalMeshParameter> SkeletalMeshParameters;
	TArray<FMutableSkeletalMeshParameter> CachedSkeletalMeshParameters;
	
	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableTextureParameter> TextureParameters;
	TArray<FMutableTextureParameter> CachedTextureParameters;

	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableMaterialParameter> MaterialParameters;
	TArray<FMutableMaterialParameter> CachedMaterialParameters;

	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableBoolParameter> BoolParameters;
	TArray<FMutableBoolParameter> CachedBoolParameters;

	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableEnumParameter> EnumParameters;
	TArray<FMutableEnumParameter> CachedEnumParameters;

	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableFloatParameter> FloatParameters;
	TArray<FMutableFloatParameter> CachedFloatParameters;

	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableVectorParameter> VectorParameters;
	TArray<FMutableVectorParameter> CachedVectorParameters;

	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableProjectorParameter> ProjectorParameters;
	TArray<FMutableProjectorParameter> CachedProjectorParameters;

	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableTransformParameter> TransformParameters;
	TArray<FMutableTransformParameter> CachedTransformParameters;
	
	UPROPERTY(EditAnywhere, Category = "Mutable Parameters", meta = (DataflowInput))
	TArray<FMutableInstancedStructParameter> InstancedStructParameters;
	TArray<FMutableInstancedStructParameter> CachedInstancedStructParameters;
	
	// Outputs

	// First option : an array of structures that contain the pair of data Component name + Skeletal Mesh
	UPROPERTY(meta = (DataflowOutput))
	TArray<FMutableGeneratedResource> GeneratedResources;
	
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<USkeletalMesh>> GeneratedSkeletalMeshes;


	// Instance generated during the runtime and used to generate the output resources
	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DisplayName = "Generated COI"))
	TObjectPtr<UCustomizableObjectInstance> CustomizableObjectInstance;

	/** Press this button to trigger first a CO compilation and then a COI update. This will generate the resources needed by the outputs of the node. */
	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DisplayName = "Generate Resources", ButtonImage = "Icons.Refresh"))
	FDataflowFunctionProperty GenerateInstanceResources;
	
	
	/** Variable that works as a semaphore to prevent the scheduling of multiple Instance Compilation + Updates*/
	bool bIsGeneratingResources = false;
	
	/**
	 * Callback method invoked when the "Generate outputs" button gets pressed. It will trigger the compilation and later update of the generated 
	 Customizable Object Instance
	 */
	void OnGenerateInstanceResourcesRequested(UE::Dataflow::FContext& Context);


	/**
	 * Get all the unique parameters defined for a given property.
	 * @tparam T The type of parameter. EX : FMutableProjectorParameter, FMutableMaterialParameter ...
	 * @param InParametersArrayProperty The property used as source to find the parameters.
	 * @param ParameterTypeName The descriptive name of the parameter type. Used for logging.
	 * @param Context The Dataflow context.
	 * @return An array with all the parameters found. Only the first occurrence of each parameter will be placed there.
	 */
	template<typename T>
	TArray<T> GetPropertyParameters(const TArray<T>* InParametersArrayProperty, const FString& ParameterTypeName, UE::Dataflow::FContext& Context)
	{
		check(InParametersArrayProperty);
		
		const TArray<T>& FoundParametersCandidates = GetValue(Context, InParametersArrayProperty);
		
		TArray<T> OutFoundParameters;
		OutFoundParameters.Reserve(FoundParametersCandidates.Num());
		
		for (const T& CandidateParameter : FoundParametersCandidates)
		{
			if (OutFoundParameters.Contains(CandidateParameter))
			{
				Context.Warning(FString::Printf( TEXT("Multiple %s Parameters have been found sharing the \"%s\" name , only the first parameter will be used."), *ParameterTypeName, *CandidateParameter.Name), this);
				continue;
			}
		
			OutFoundParameters.Add(CandidateParameter);
		}
		
		return OutFoundParameters;
	}
	
	/**
	 * Evaluate the input nodes and add their data to our local caches so we can later access it without the dataflow context
	 */
	void CacheNodeInputs(UE::Dataflow::FContext& Context);
	
	/**
	 * Apply the values in the cached parameters into the provided Instance
	 * @param Instance The instance that will get its parameters updated
	 */
	void ApplyInstanceParameters(UCustomizableObjectInstance& Instance);

	/**
	 * Empty the caches holding the data provided by the evaluation of the input pins
	 */
	void ClearCachedParameters();
	
	/**
	 * Request the compilation of the CO so we can later generate a Customizable Object Instance.
	 */
	void RequestCompilation(UE::Dataflow::FContext& Context);
	
	/** 
	 * Request the Update of the given COI.
	 */
	void RequestUpdate(UCustomizableObjectInstance& Instance);

	/**
	 * Extract from the given Customizable Object Instance the generated skeletal meshes for each one of the components the instance has.
	 * @param Instance The COI we want to get the data from
	 * @return An array of structures composed by the component name and the skeletal mesh generated for said component..
	 */
	TArray<FMutableGeneratedResource> GetInstanceGeneratedMeshes(const UCustomizableObjectInstance& Instance) const;
};



