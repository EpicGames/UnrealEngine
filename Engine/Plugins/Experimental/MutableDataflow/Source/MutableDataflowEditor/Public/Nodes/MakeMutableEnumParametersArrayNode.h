// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"
#include "CoreMinimal.h"
#include "MakeMutableParametersArrayBaseNode.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowInputOutput.h"
#include "MutableDataflowParameters.h"

#include "MakeMutableEnumParametersArrayNode.generated.h"


USTRUCT(meta = (Experimental))
struct FMakeMutableEnumParametersArrayNode : public FMakeMutableParametersArrayBaseNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeMutableEnumParametersArrayNode, "MakeMutableEnumParametersArray", "Mutable|Parameters", "")

private:
	UPROPERTY()
	TArray<FMutableEnumParameter> InputEnumParameters;
	
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Enum Parameters"))
	TArray<FMutableEnumParameter> GroupedParameters;

public:
	
	FMakeMutableEnumParametersArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	
	// Overrides to allow the addition and removal of optional inputs
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanRemovePin() const override;
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override; 
};