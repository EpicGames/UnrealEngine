// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"
#include "CoreMinimal.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowInputOutput.h"

#include "MakeMutableParametersArrayBaseNode.generated.h"


USTRUCT(meta = (Experimental))
struct FMakeMutableParametersArrayBaseNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeMutableParametersArrayBaseNode, "MakeParametersArray", "Mutable|Parameters", "")
	
public:
	FMakeMutableParametersArrayBaseNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

protected:

	/**
	 * The min amount of input pins the node can have. 
	 */
	static constexpr uint16 MinRequiredInputsCount = 1;

	template<typename T>
	void AddDefaultInputs(TArray<T>& InParametersArray);

	template<typename T>
	void EvaluateParameterNode(const TArray<T>& InputParameters, const TArray<T>& OutputParameters, UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const;

	template<typename T>
	UE::Dataflow::TConnectionReference<T> GetConnectionReference(const TArray<T>& ParametersArray, int32 Index) const;

	template<typename T>
	TArray<UE::Dataflow::FPin> AddParameterPin(TArray<T>& InOutParametersArray);
	
	template<typename T>
	bool CanRemoveParameterPin(const TArray<T>& ParametersArray) const;

	template<typename T>
	TArray<UE::Dataflow::FPin> GetParameterPinsToRemove(const TArray<T>& ParametersArray) const;;
	
	template<typename T>
	void OnParameterPinRemoved(TArray<T>& InParametersArray, const UE::Dataflow::FPin& Pin);

	template<typename T>
	void PostNodeSerialize(const TArray<T>& ParametersArray, const FArchive& Ar);

	// FDataflowNode
	virtual bool CanAddPin() const override;
};


inline FMakeMutableParametersArrayBaseNode::FMakeMutableParametersArrayBaseNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
: Super(InParam, InGuid)
{
}

inline bool FMakeMutableParametersArrayBaseNode::CanAddPin() const
{
	return true;
}


template <typename T>
void FMakeMutableParametersArrayBaseNode::AddDefaultInputs(TArray<T>& InParametersArray)
{
	for (int32 Index = 0; Index < MinRequiredInputsCount; ++Index)
	{
		AddParameterPin(InParametersArray);
	}
}


template <typename T>
void FMakeMutableParametersArrayBaseNode::EvaluateParameterNode(const TArray<T>& InputParameters, const TArray<T>& OutputParameters,  UE::Dataflow::FContext& Context,
	const FDataflowOutput* Out) const
{
	if (Out->IsA(&OutputParameters))
	{
		// Grab the values from the inputs
		TArray<T> Output;
		for (const T& Parameter : InputParameters)
		{
			Output.Add(GetValue(Context, &Parameter));
		}

		// Provide the composed array as an output
		SetValue(Context, Output, &OutputParameters);
	}
}


template<typename T>
UE::Dataflow::TConnectionReference<T> FMakeMutableParametersArrayBaseNode::GetConnectionReference(const TArray<T>& ParametersArray, int32 Index) const
{
	return { &ParametersArray[Index], Index, &ParametersArray };
}


template <typename T>
TArray<UE::Dataflow::FPin> FMakeMutableParametersArrayBaseNode::AddParameterPin(TArray<T>& InOutParametersArray)
{
	const int32 Index = InOutParametersArray.AddDefaulted();
	UE::Dataflow::TConnectionReference<T> ConnectionRef = GetConnectionReference(InOutParametersArray, Index);
	const FDataflowInput& Input = RegisterInputArrayConnection(ConnectionRef);
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}


template <typename T>
bool FMakeMutableParametersArrayBaseNode::CanRemoveParameterPin(const TArray<T>& ParametersArray) const
{
	return ParametersArray.Num() > MinRequiredInputsCount;
}


template <typename T>
TArray<UE::Dataflow::FPin> FMakeMutableParametersArrayBaseNode::GetParameterPinsToRemove(const TArray<T>& ParametersArray) const
{
	const int32 Index = ParametersArray.Num() - 1;
	check(ParametersArray.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(ParametersArray, Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}


template <typename T>
void FMakeMutableParametersArrayBaseNode::OnParameterPinRemoved(TArray<T>& InParametersArray, const UE::Dataflow::FPin& Pin)
{
	const int32 Index = InParametersArray.Num() - 1;
	check(InParametersArray.IsValidIndex(Index));
#if DO_CHECK
		const FDataflowInput* const Input = FindInput(GetConnectionReference(InParametersArray, Index));
		check(Input);
		check(Input->GetName() == Pin.Name);
		check(Input->GetType() == Pin.Type);
#endif
	InParametersArray.SetNum(Index);
}


template <typename T>
void FMakeMutableParametersArrayBaseNode::PostNodeSerialize(const TArray<T>& ParametersArray, const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		// Make sure the node does have the same amount of inputs as when saved
		for (int32 Index = 0; Index < ParametersArray.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(ParametersArray, Index));
		}
	}
}
