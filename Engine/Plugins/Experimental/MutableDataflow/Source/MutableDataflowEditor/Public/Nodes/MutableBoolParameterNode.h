// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowNode.h"
#include "MutableDataflowParameters.h"
#include "MutableParameterNode.h"
#include "Misc/Guid.h"
#include "CoreMinimal.h"

#include "MutableBoolParameterNode.generated.h"


USTRUCT(meta = (Experimental))
struct FMutableBoolParameterNode : public FMutableParameterNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMutableBoolParameterNode, "MutableBoolParameterNode", "Mutable|Parameters", "")

	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DataflowInput))
	bool Bool;
	
	UPROPERTY(meta = (DataflowOutput))
	FMutableBoolParameter BoolParameter;

public:
	
	FMutableBoolParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};