// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowNode.h"
#include "MutableDataflowParameters.h"
#include "MutableParameterNode.h"
#include "Misc/Guid.h"
#include "CoreMinimal.h"

#include "MutableEnumParameterNode.generated.h"


USTRUCT(meta = (Experimental))
struct FMutableEnumParameterNode : public FMutableParameterNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMutableEnumParameterNode, "MutableEnumParameterNode", "Mutable|Parameters", "")

	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DataflowInput))
	FString EnumOption;
	
	UPROPERTY(meta = (DataflowOutput))
	FMutableEnumParameter EnumParameter;

public:
	
	FMutableEnumParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};