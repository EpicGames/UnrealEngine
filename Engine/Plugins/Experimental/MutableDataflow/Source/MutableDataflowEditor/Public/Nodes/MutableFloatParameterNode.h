// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowNode.h"
#include "MutableDataflowParameters.h"
#include "MutableParameterNode.h"
#include "Misc/Guid.h"
#include "CoreMinimal.h"

#include "MutableFloatParameterNode.generated.h"


USTRUCT(meta = (Experimental))
struct FMutableFloatParameterNode : public FMutableParameterNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMutableFloatParameterNode, "MutableFloatParameterNode", "Mutable|Parameters", "")

	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DataflowInput))
	float Float;
	
	UPROPERTY(meta = (DataflowOutput))
	FMutableFloatParameter FloatParameter;

public:
	
	FMutableFloatParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};