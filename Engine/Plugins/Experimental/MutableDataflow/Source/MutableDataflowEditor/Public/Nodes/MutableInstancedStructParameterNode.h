// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowAnyType.h"
#include "Misc/Guid.h"
#include "CoreMinimal.h"
#include "MutableDataflowParameters.h"
#include "MutableParameterNode.h"

#include "MutableInstancedStructParameterNode.generated.h"


USTRUCT(meta = (Experimental))
struct FMutableInstancedStructParameterNode : public FMutableParameterNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMutableInstancedStructParameterNode, "MutableInstancedStructParameter", "Mutable|Parameters", "")
	
	UPROPERTY(meta=(DataflowInput))
	FDataflowUStructTypes InputStruct;
	
	UPROPERTY(meta = (DataflowOutput))
	FMutableInstancedStructParameter InstancedStructParameter;

public:
	
	FMutableInstancedStructParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
