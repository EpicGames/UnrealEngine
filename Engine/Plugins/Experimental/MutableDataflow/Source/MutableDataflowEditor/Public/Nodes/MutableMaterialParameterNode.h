// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowNode.h"
#include "Misc/Guid.h"
#include "CoreMinimal.h"
#include "MutableDataflowParameters.h"
#include "MutableParameterNode.h"

#include "MutableMaterialParameterNode.generated.h"

USTRUCT(meta = (Experimental))
struct FMutableMaterialParameterNode : public FMutableParameterNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMutableMaterialParameterNode, "MutableMaterialParameter", "Mutable|Parameters", "")

	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DataflowInput))
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY(meta = (DataflowOutput))
	FMutableMaterialParameter MaterialParameter;

public:
	
	FMutableMaterialParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};