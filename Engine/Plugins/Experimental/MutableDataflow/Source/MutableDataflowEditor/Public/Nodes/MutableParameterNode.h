// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowNode.h"
#include "Misc/Guid.h"
#include "CoreMinimal.h"

#include "MutableParameterNode.generated.h"


USTRUCT(meta = (Experimental))
struct FMutableParameterNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMutableParameterNode, "MutableParameterBaseNode", "Mutable|Parameters", "")

	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DataflowInput))
	FString ParameterName;
	
public:
	
	FMutableParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

protected:
	FString GetParameterName(UE::Dataflow::FContext& Context) const;
};