// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Dataflow/DataflowNode.h"
#include "MutableDataflowParameters.h"
#include "MutableParameterNode.h"
#include "Misc/Guid.h"
#include "CoreMinimal.h"

#include "MutableSkeletalMeshParameterNode.generated.h"


USTRUCT(meta = (Experimental))
struct FMutableSkeletalMeshParameterNode : public FMutableParameterNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMutableSkeletalMeshParameterNode, "MutableSkeletalMeshParameter", "Mutable|Parameters", "")

	UPROPERTY(EditAnywhere, Category = "Mutable", meta = (DataflowInput))
	TObjectPtr<USkeletalMesh> SkeletalMesh;
	
	UPROPERTY(meta = (DataflowOutput))
	FMutableSkeletalMeshParameter SkeletalMeshParameter;

public:
	
	FMutableSkeletalMeshParameterNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
